// Copyright Epic Games, Inc. All Rights Reserved.

#include "ToolboxModule.h"
#include "Textures/SlateIcon.h"
#include "Modules/ModuleManager.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "SlateOptMacros.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Docking/TabManager.h"
#include "EditorStyleSet.h"
#include "GammaUI.h"
#include "ModuleUIInterface.h"
#include "DesktopPlatformModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Testing/STestSuite.h"
#include "Widgets/Testing/SStarshipSuite.h"
#include "Editor/WorkspaceMenuStructure/Public/WorkspaceMenuStructure.h"
#include "Editor/WorkspaceMenuStructure/Public/WorkspaceMenuStructureModule.h"
#include "ISourceCodeAccessor.h"
#include "ISourceCodeAccessModule.h"
#include "Misc/CommandLine.h"

#if !UE_BUILD_SHIPPING
#include "ISlateReflectorModule.h"
#endif // #if !UE_BUILD_SHIPPING


